{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "IOS TapJoy Advertising Provider",
	"Description" : "IOS TapJoy Provider",
	"Category" : "Advertising",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "IOSTapJoy",
			"Type" : "Runtime",
			"LoadingPhase" : "Default",
			"PlatformAllowList" :
			[
				"IOS"
			]
		}
	]
}