// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class IOSTapJoy : ModuleRules
	{
		public IOSTapJoy( ReadOnlyTargetRules Target ) : base(Target)
		{
			bRequiresPlatformSDK = true;
		
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					// ... add other public dependencies that you statically link with here ...
				}
				);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Advertising",
					"ApplicationCore",
					// ... add private dependencies that you statically link with here ...
				}
				);

			PublicIncludePathModuleNames.Add( "Advertising" );

			// Add the Tapjoy framework
			PublicAdditionalFrameworks.Add( 
				new Framework( 
					"Tapjoy",														// Framework name
					"../../ThirdPartyFrameworks/Tapjoy.embeddedframework.zip",		// Zip name
					"Resources/TapjoyResources.bundle"								// Resources we need copied and staged
				)
			); 

			PublicFrameworks.AddRange( 
				new string[] 
				{ 
					"EventKit",
					"MediaPlayer",
					"AdSupport",
					"CoreLocation",
					"SystemConfiguration",
					"MessageUI",
					"Security",
					"CoreTelephony",
					"Twitter",
					"Social"
				}
				);
		}
	}
}
