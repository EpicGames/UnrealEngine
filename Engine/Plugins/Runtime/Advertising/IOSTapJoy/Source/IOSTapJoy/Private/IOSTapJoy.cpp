// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Apple/AppleStringUtils.h"
#include "Interfaces/IAdvertisingProvider.h"
#include "IOS/IOSAppDelegate.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Misc/ConfigCacheIni.h"
#import <Tapjoy/Tapjoy.h>

class FTapJoyProvider : public IAdvertisingProvider
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/** IAdvertisingProvider implementation */
	virtual void ShowAdBanner(bool bShowOnBottomOfScreen, int32 adID) override;
	virtual void HideAdBanner() override;
	virtual void CloseAdBanner() override;
	virtual int32 GetAdIDCount() override;

	virtual void LoadInterstitialAd(int32 adID) override;
	virtual bool IsInterstitialAdAvailable() override;
	virtual bool IsInterstitialAdRequested() override;
	virtual void ShowInterstitialAd() override;
};

IMPLEMENT_MODULE( FTapJoyProvider, IOSTapJoy )

static FString AppIDString( TEXT( "" ) );
static FString SecretKeyString( TEXT( "" ) );
static FString CurrencyString( TEXT( "" ) );

@interface IOSTapJoy : UIResponder< TJCAdDelegate >
@end

@implementation IOSTapJoy

static bool bAttemptingToShowAd	= false;

+ (IOSTapJoy*)GetDelegate
{
	static IOSTapJoy * Singleton = [[IOSTapJoy alloc] init];
	return Singleton;
}

-(void)tjcConnectSuccess:(NSNotification*)notifyObj
{
	NSLog(@"Tapjoy connect Succeeded");
}


- (void)tjcConnectFail:(NSNotification*)notifyObj
{
	NSLog(@"Tapjoy connect Failed");
}

- (void)StartupTapJoy
{
	// Tapjoy Connect Notifications
	[[NSNotificationCenter defaultCenter] addObserver:self
		selector:@selector(tjcConnectSuccess:)
		name:TJC_CONNECT_SUCCESS
		object:nil];

	[[NSNotificationCenter defaultCenter] addObserver:self
		selector:@selector(tjcConnectFail:)
		name:TJC_CONNECT_FAILED
		object:nil];

	// This code will execute after your module is loaded into memory (but after global variables are initialized, of course.)
	[Tapjoy requestTapjoyConnect:[NSString stringWithFString:AppIDString]
		secretKey:[NSString stringWithFString:SecretKeyString]
		options:@{ TJC_OPTION_ENABLE_LOGGING : @(YES) }
	// If you are not using Tapjoy Managed currency, you would set your own user ID here.
	// TJC_OPTION_USER_ID : @"A_UNIQUE_USER_ID"

	// You can also set your event segmentation parameters here.
	// Example segmentationParams object -- NSDictionary *segmentationParams = @{@"iap" : @(YES)};
	// TJC_OPTION_SEGMENTATION_PARAMS : segmentationParams
	];
}

-(void)ShowAdBanner:(NSNumber*)bShowOnBottomOfScreen
{
	if ( bAttemptingToShowAd )
	{
		NSLog( @"ShowAdBanner: Already attempting to show ad..." );
		return;
	}

	if ( !CurrencyString.IsEmpty() )
	{
		NSString * CurrencyStringNS = [NSString stringWithFString:CurrencyString];

		//NSLog( @"ShowAdBanner: %@", CurrencyStringNS );

		[Tapjoy getDisplayAdWithDelegate:self currencyID:CurrencyStringNS];
	}
	else
	{
		[Tapjoy getDisplayAdWithDelegate:self];
	}

	bAttemptingToShowAd = true;
}

-(void)HideAdBanner
{
	TJCAdView * adView = nil;

	for ( UIView * SubView in [[IOSAppDelegate GetDelegate].RootView subviews] ) 
	{
		if ( [SubView isKindOfClass:[TJCAdView class]] ) 
		{
			adView = (TJCAdView*)SubView;
			break;
		}
	}

	if ( adView == nil )
	{
		NSLog( @"HideAdBanner: No ad view is active..." );
		return;
	}

	// Fade it out
	//if ( ![Tapjoy getDisplayAdView].hidden )
	if ( !adView.hidden )
	{
		[UIView animateWithDuration:0.4f 
			animations:^
			{ 
				adView.alpha = 0.0f; 
			} 
			completion:^(BOOL finished) 
			{ 
				adView.hidden = YES; 
			}
		];
	}
}

- (void)didReceiveAd:(TJCAdView*)adView
{
	NSLog( @"didReceiveAd called..." );

	if ( !bAttemptingToShowAd )
	{
		NSLog( @"didReceiveAd: bAttemptingToShowAd == false" );
	}

	// Remove any existing ad views
	for ( UIView * SubView in [[IOSAppDelegate GetDelegate].RootView subviews] ) 
	{
		if ( [SubView isKindOfClass:[TJCAdView class]] ) 
		{
			[SubView removeFromSuperview];
		}
	}

	// Hide it initially
	adView.hidden = YES;
	adView.alpha = 0.0f;

	// Add it it the main view
	[[IOSAppDelegate GetDelegate].RootView addSubview : adView];

	// Fade it in
	if ( adView.hidden )
	{
		adView.hidden = NO;
		[UIView animateWithDuration:0.4f
			animations:^
			{
				adView.alpha = 1.0f;
			}
		];
	}

	bAttemptingToShowAd = false;
}

- (void)didFailWithMessage:(NSString*)msg
{
	if ( !bAttemptingToShowAd )
	{
		NSLog( @"didFailWithMessage: bAttemptingToShowAd == false" );
	}

	NSLog( @"didFailWithMessage: %@", msg );
	bAttemptingToShowAd = false;
}

- (NSString*)adContentSize
{
	return TJC_DISPLAY_AD_SIZE_320X50;
}

- (BOOL)shouldRefreshAd
{
	return NO;
}
@end

void FTapJoyProvider::StartupModule() 
{
	GConfig->GetString( TEXT( "TapJoy" ), TEXT( "AppID" ),			AppIDString,		GEngineIni );
	GConfig->GetString( TEXT( "TapJoy" ), TEXT( "SecretKey" ),		SecretKeyString,	GEngineIni );
	GConfig->GetString( TEXT( "TapJoy") , TEXT( "CurrencyString" ),	CurrencyString,		GEngineIni );

	[[IOSTapJoy GetDelegate] performSelectorOnMainThread:@selector(StartupTapJoy) withObject:nil waitUntilDone : NO];
}

void FTapJoyProvider::ShutdownModule()
{
}

void FTapJoyProvider::ShowAdBanner(bool bShowOnBottomOfScreen, int32 /*adID*/)
{
	[[IOSTapJoy GetDelegate] performSelectorOnMainThread:@selector(ShowAdBanner:) withObject:[NSNumber numberWithBool : bShowOnBottomOfScreen] waitUntilDone : NO];
}

void FTapJoyProvider::HideAdBanner()
{
	[[IOSTapJoy GetDelegate] performSelectorOnMainThread:@selector(HideAdBanner) withObject:nil waitUntilDone : NO];
}

void FTapJoyProvider::CloseAdBanner()
{
	HideAdBanner();
}

int32 FTapJoyProvider::GetAdIDCount()
{
	return 1;
}

void LoadInterstitialAd(int32 adID)
{

}

bool IsInterstitialAdAvailable()
{
	return false;
}

bool IsInterstitialAdRequested()
{
	return false;
}

void ShowInterstitialAd()
{
	
}

